import os
import csv
import re

# Directory containing the C++ header files
base_directory = '../Source/TextureGraph/Public'

# Regular expression pattern to match class names
class_pattern = r'class TEXTUREGRAPH_API UTG_Expression_([A-Za-z_][A-Za-z0-9_]*)\s*(?::\s*public\s*([A-Za-z_][A-Za-z0-9_]*))?\s*\{\s*((?:.|\n)*?)\s*\};'
category_pattern = r'TG_Category::([A-Za-z_]+)'
uxname_pattern = r'GetDefaultName\(\)[^{]*\{[^\"]*\"([^"]+)\"'

tooltip_pattern = r'GetTooltipText\(\)[^{]*\{[^\"]*\"([^"]+)\"'

tooltip_pattern = r'GetTooltipText\(\)[^{]*\{[^\"]*\"([^"]+)\"'

property_pattern = r'UPROPERTY\((.*)\)\s+((.*;))'


# CSV file to write the results
csv_file = 'class_info.csv'

# Initialize a list to store the results
class_info = []

def csv_header():
    return ("ID",    "Class",       "Member",   "UI Name",    "Category",   "Type",          "Default Value",    "Tooltip",    "parent",     "sub_items",    "file_path")


def make_class_entry(class_name, class_base, category, file_path, directory_name, uxname, tooltip, sub_items):
    if (class_base == "UTG_Expression"):
        class_base = ""
    return (class_name,  class_name,    "",        uxname,       category,     "",              "",                 tooltip,       class_base,  sub_items,      file_path)

    
def make_member_entry(prop_id, member, uxname, category, prop_type, default_value, class_name):
    return (prop_id,     "",            member,    uxname,       category,     prop_type,       default_value,      "",            class_name,  "",             "")


# Function to search for class names in a file and add them to the list
def search_class_names(file_path):
    with open(file_path, 'r', encoding='utf-8') as file:
        content = file.read()
        #components = file_path.split(os.path.sep)
        #if len(components) > 2:
        #    directory_name = components[-2]
        #else:
        #    directory_name = None
        directory_name = os.path.basename(os.path.dirname(file_path))
        class_matches = re.finditer(class_pattern, content)
        #class_matches = re.finditer(classcontent_pattern, content)
        for class_match in class_matches:
            class_name = class_match.group(1)
            class_base = class_match.group(2)
            class_block = class_match.group(3)
            
            macro_matches = re.findall(category_pattern, class_block)
            category = ""
            for macro_match in macro_matches:
                category = macro_match

            uxname_matches = re.findall(uxname_pattern, class_block)
            uxname = class_name #if no ux name is found probably we use the actual class name
            for uxname_match in uxname_matches:
                uxname = uxname_match

            tooltip_matches = re.findall(tooltip_pattern, class_block)
            tooltip = ""
            for tooltip_match in tooltip_matches:
                tooltip = tooltip_match

            class_name = 'UTG_Expression_' + class_name

            sub_items = ''
            class_props = []
            prop_matches = re.findall(property_pattern, class_block)
            for prop_match in prop_matches:
                prop_attribs = prop_match[0]
                member_declaration = prop_match[1]
                
                #print(f"    Prop {prop_attribs} Declaration = {member_declaration}")

                member_match = re.findall(r'([A-Za-z0-9_<>]*)\s*([^=]*)(?: *= *([^;]+))?;', member_declaration)
                prop_type = member_match[0][0]
                member_name = member_match[0][1]
                default_value =  member_match[0][2]  


                prop_category_matches = re.findall(r'Category\s*=\s*"([^"]+)"', prop_attribs)
                prop_category = ""
                if (len(prop_category_matches)):
                    prop_category = prop_category_matches[0]

                prop_display_matches = re.findall(r'DisplayName\s*=\s*"([^"]+)"', prop_attribs)
                prop_display = member_name
                if (len(prop_display_matches)):
                    prop_display = prop_display_matches[0]

                prop_meta_matches = re.findall(r'meta\s*=\s*"([^"]+)"', prop_attribs)
                prop_meta = ""
                if (len(prop_meta_matches)):
                    prop_meta = prop_meta_matches[0]

                
                prop_id = class_name + '.' + member_name
                sub_items += prop_id + ' '

                #print(f"    Prop {prop_display} pin = {member_name} category = {prop_category} type = {prop_type} default = {default_value}")
                class_props.append(make_member_entry(prop_id, member_name, prop_display, prop_category, prop_type, default_value, class_name))

            #print(f"Found {class_name} base = {class_base} category = {category} folder = {directory_name} uxname = {uxname}")            
            class_info.append(make_class_entry(class_name, class_base, category, file_path, directory_name, uxname, tooltip, sub_items))
            for sub in class_props:
                class_info.append(sub)


            

# Traverse the directory and its subdirectories
for foldername, subfolders, filenames in os.walk(base_directory):
    for filename in filenames:
        if filename.endswith('.h'):
            file_path = os.path.join(foldername, filename)
            search_class_names(file_path)

# Write the results to a CSV file
with open(csv_file, 'w', newline='') as csvfile:
    csv_writer = csv.writer(csvfile)
    csv_writer.writerow(csv_header())
    for entry in class_info:
        csv_writer.writerow(entry)

print(f"Found {len(class_info)} entries. Data written to {csv_file}.")
